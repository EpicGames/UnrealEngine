// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateGraph.h"

#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Parse.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogStateGraph);

class FStateGraphModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FStateGraphModule, StateGraph);

namespace UE {

// FStateGraphNode

FStateGraphNode::FStateGraphNode(FName InName) :
	Name(InName),
	LogName(Name.ToString())
{
	UE_LOGF(LogStateGraph, VeryVerbose, "[%ls] Created node", *LogName);
}

FStateGraphNode::~FStateGraphNode()
{
	UE_LOGF(LogStateGraph, VeryVerbose, "[%ls] Destroyed node", *LogName);
}

const TCHAR* FStateGraphNode::GetStatusName(EStatus Status)
{
	switch (Status)
	{
		case EStatus::NotStarted: return TEXT("NotStarted");
		case EStatus::Blocked: return TEXT("Blocked");
		case EStatus::Started: return TEXT("Started");
		case EStatus::Completed: return TEXT("Completed");
		case EStatus::TimedOut: return TEXT("TimedOut");
		default: checkNoEntry(); return TEXT("Unknown");
	}
}

void FStateGraphNode::SetTimeout(double InTimeout)
{
	if (FStateGraph::IsNoTimeouts())
	{
		return;
	}
	Timeout = InTimeout;
}

double FStateGraphNode::GetDuration() const
{
	if (StartTime.GetTicks() == 0.f)
	{
		return 0.f;
	}

	return ((CompletedTime.GetTicks() == 0.f ? FDateTime::UtcNow() : CompletedTime) - StartTime).GetTotalSeconds();
}

bool FStateGraphNode::CheckDependencies() const
{
	FStateGraphPtr StateGraphPtr(StateGraphWeakPtr.Pin());
	if (!StateGraphPtr.IsValid())
	{
		UE_LOGF(LogStateGraph, Warning, "[%ls] Node checked with invalid state graph", *LogName);
		return false;
	}

	for (FName Dependency : Dependencies)
	{
		FStateGraphNodeRef* DependencyNode = StateGraphPtr->GetNodeRef(Dependency);
		if (!DependencyNode || (*DependencyNode)->GetStatus() != FStateGraphNode::EStatus::Completed)
		{
			return false;
		}
	}

	return true;
}

void FStateGraphNode::Complete()
{
	if (Status == EStatus::Completed)
	{
		UE_LOGF(LogStateGraph, Warning, "[%ls] Node already completed", *LogName);
		return;
	}

	CompletedTime = FDateTime::UtcNow();
	if (Status == EStatus::NotStarted)
	{
		StartTime = CompletedTime;
	}

	UE_LOGF(LogStateGraph, Log, "[%ls] Completed node (Duration=%.3f Timeout=%.3f)", *LogName, GetDuration(), Timeout);

	// Keep a reference to check if the node is destroyed during external functions.
	FStateGraphNodeWeakPtr StateGraphNodeWeakPtr(AsWeak());

	SetStatus(EStatus::Completed);

	if (!StateGraphNodeWeakPtr.IsValid())
	{
		return;
	}

	// Run state graph again since this completion may have fulfilled dependencies to start new nodes.
	FStateGraphPtr StateGraphPtr = StateGraphWeakPtr.Pin();
	if (StateGraphPtr.IsValid())
	{
		if (StateGraphPtr->GetStatus() != FStateGraph::EStatus::Paused)
		{
			StateGraphPtr->Run();
		}
	}
	else
	{
		UE_LOGF(LogStateGraph, Warning, "[%ls] Node completed with invalid state graph", *LogName);
	}
}

void FStateGraphNode::Reset()
{
	UE_LOGF(LogStateGraph, Verbose, "[%ls] Resetting node", *LogName);
	StartTime = 0.f;
	CompletedTime = 0.f;
	SetStatus(EStatus::NotStarted);
}

void FStateGraphNode::UpdateConfig()
{
	double RequestedTimeout = 0.;
	GConfig->GetDouble(*ConfigSectionName, TEXT("Timeout"), RequestedTimeout, GEngineIni);
	SetTimeout(RequestedTimeout);
}

void FStateGraphNode::SetStatus(EStatus NewStatus)
{
	if (Status != NewStatus)
	{
		const EStatus OldStatus = Status;
		Status = NewStatus;
		FStateGraphPtr StateGraphPtr = StateGraphWeakPtr.Pin();
		if (StateGraphPtr.IsValid())
		{
			StateGraphPtr->OnNodeStatusChanged.Broadcast(*this, OldStatus, NewStatus);
		}
	}
}

// FStateGraphNodeFunction

FStateGraphNodeFunction::FStateGraphNodeFunction(FName InName, const FStateGraphNodeFunctionStart& InStartFunction) :
	FStateGraphNode(InName),
	StartFunction(InStartFunction)
{}

bool FStateGraphNodeFunction::CheckDependencies() const
{
	if (!StartFunction.IsBound())
	{
		UE_LOGF(LogStateGraph, Warning, "[%ls] Function node start not bound", *GetLogName());
		return false;
	}

	return FStateGraphNode::CheckDependencies();
}

void FStateGraphNodeFunction::Start()
{
	// CheckDependencies verified the function is bound right before calling this in StateGraph->Run().
	check(StartFunction.IsBound());

	FStateGraph* StateGraph = GetStateGraph().Get();
	check(StateGraph); // Must be valid since it is starting this node.
	FStateGraphNodeWeakPtr NodeWeakPtr(AsWeak());
	StartFunction.Execute(*StateGraph,
		[NodeWeakPtr, LogName = GetLogName()]()
		{
			FStateGraphNodePtr NodePtr = NodeWeakPtr.Pin();
			if (NodePtr)
			{
				NodePtr->Complete();
			}
			else
			{
				UE_LOGF(LogStateGraph, Log, "[%ls] Function node completed after node was destroyed", *LogName);
			}
		});
}

// FStateGraph

FStateGraph::FStateGraph(FName InName, const FString& InContextName) :
	Name(InName),
	ContextName(InContextName),
	ConfigSectionName(FString::Printf(TEXT("StateGraph.%s"), *InName.ToString()))
{
	if (ContextName.IsEmpty())
	{
		LogName = Name.ToString();
	}
	else
	{
		LogName = FString::Printf(TEXT("%s(%s)"), *Name.ToString(), *ContextName);
	}

	UE_LOGF(LogStateGraph, VeryVerbose, "[%ls] Created state graph", *LogName);
}

FStateGraph::~FStateGraph()
{
	UE_LOGF(LogStateGraph, VeryVerbose, "[%ls] Destroyed state graph", *LogName);
	FCoreDelegates::TSOnConfigSectionsChanged().Remove(ConfigSectionsChangedDelegate);
}

void FStateGraph::Initialize()
{
	if (!ConfigSectionsChangedDelegate.IsValid())
	{
		ConfigSectionsChangedDelegate = FCoreDelegates::TSOnConfigSectionsChanged().AddSP(this, &FStateGraph::OnConfigSectionsChanged);
	}

	UpdateConfig();
}

const TCHAR* FStateGraph::GetStatusName(EStatus Status)
{
	switch (Status)
	{
		case EStatus::NotStarted: return TEXT("NotStarted");
		case EStatus::Running: return TEXT("Running");
		case EStatus::Waiting: return TEXT("Waiting");
		case EStatus::Blocked: return TEXT("Blocked");
		case EStatus::Completed: return TEXT("Completed");
		case EStatus::Paused: return TEXT("Paused");
		case EStatus::TimedOut: return TEXT("TimedOut");
		default: checkNoEntry(); return TEXT("Unknown");
	}
}

void FStateGraph::SetTimeout(double InTimeout)
{
	if (FStateGraph::IsNoTimeouts())
	{
		return;
	}
	Timeout = InTimeout;
}

double FStateGraph::GetDuration() const
{
	if (StartTime == 0.f)
	{
		return 0.f;
	}

	return ((CompletedTime.GetTicks() == 0.f ? FDateTime::UtcNow() : CompletedTime) - StartTime).GetTotalSeconds();
}

bool FStateGraph::AddNode(const FStateGraphNodeRef& Node)
{
	if (Node->StateGraphWeakPtr.IsValid())
	{
		UE_LOGF(LogStateGraph, Warning, "[%ls.%ls] Node already associated with a state graph %ls", *LogName, *Node->GetName().ToString(), *Node->LogName);
		return false;
	}

	if (Nodes.Contains(Node->GetName()))
	{
		UE_LOGF(LogStateGraph, Warning, "[%ls.%ls] Node with the same name already exists", *LogName, *Node->GetName().ToString());
		return false;
	}

	Node->LogName = FString::Printf(TEXT("%s.%s"), *LogName, *Node->GetName().ToString());
	UE_LOGF(LogStateGraph, Verbose, "[%ls] Adding node", *Node->LogName);
	Node->StateGraphWeakPtr = AsWeak();
	Node->ConfigSectionName = FString::Printf(TEXT("%s.%s"), *ConfigSectionName, *Node->GetName().ToString());
	Nodes.Add(Node->GetName(), Node);
	Node->UpdateConfig();
	return true;
}

bool FStateGraph::RemoveNode(FName NodeName)
{
	FStateGraphNodeRef* Node = GetNodeRef(NodeName);
	if (!Node)
	{
		UE_LOGF(LogStateGraph, Warning, "[%ls.%ls] Failed to remove node", *LogName, *NodeName.ToString());
		return false;
	}

	UE_LOGF(LogStateGraph, Verbose, "[%ls] Removing node", *(*Node)->LogName);
	(*Node)->LogName = (*Node)->GetName().ToString();
	(*Node)->StateGraphWeakPtr.Reset();

	// Keep a reference to check if the state graph is destroyed during external functions.
	FStateGraphWeakPtr StateGraphWeakPtr(AsWeak());
	(*Node)->Removed();
	if (StateGraphWeakPtr.IsValid())
	{
		Nodes.Remove(NodeName);
	}

	return true;
}

void FStateGraph::RemoveAllNodes()
{
	// Keep a reference to check if the state graph is destroyed during external functions.
	FStateGraphWeakPtr StateGraphWeakPtr(AsWeak());

	// Copy names since node map may be modified during loop.
	TArray<FName> NodeNames;
	Nodes.GenerateKeyArray(NodeNames);
	for (FName NodeName : NodeNames)
	{
		RemoveNode(NodeName);
		if (!StateGraphWeakPtr.IsValid())
		{
			return;
		}
	}
}

bool FStateGraph::AddDependencies(FName NodeName, const TArrayView<const FName> Dependencies)
{
	UE::FStateGraphNodePtr Node = GetNode(NodeName);
	if (Node)
	{
		Node->Dependencies.Append(Dependencies);
		return true;
	}

	return false;
}

void FStateGraph::Run()
{
	FDateTime Now = FDateTime::UtcNow();

	if (Status == EStatus::NotStarted)
	{
		StartTime = Now;
	}

	// Keep a reference to check if the state graph is destroyed during external functions.
	FStateGraphWeakPtr StateGraphWeakPtr(AsWeak());

	SetStatus(EStatus::Running);

	if (!StateGraphWeakPtr.IsValid())
	{
		return;
	}

	Now = FDateTime::UtcNow();

	if (bRunning)
	{
		bRunAgain = true;
		return;
	}

	FTSTicker::RemoveTicker(TimeoutTicker);

	double NextTimeout = 0.f;
	if (Timeout > 0.f)
	{
		const double Duration = (Now - StartTime).GetTotalSeconds();
		NextTimeout = Timeout - Duration;
		if (NextTimeout <= 0.f)
		{
			UE_LOGF(LogStateGraph, Log, "[%ls] State graph timed out (Duration=%.3f Timeout=%.3f)", *LogName, Duration, Timeout);
			SetStatus(EStatus::TimedOut);
			return;
		}
	}

	bRunning = true;

	UE_LOGF(LogStateGraph, Verbose, "[%ls] Starting run loop", *LogName);

	uint32 Blocked = 0;
	uint32 Started = 0;
	uint32 Running = 0;
	uint32 Completed = 0;
	uint32 Removed = 0;
	uint32 TimedOut = 0;

	// Copy names since node map may be modified during loop.
	TArray<FName> NodeNames;
	Nodes.GenerateKeyArray(NodeNames);
	for (FName NodeName : NodeNames)
	{
		if (Status != EStatus::Running)
		{
			// State graph was reset or paused during last node Start().
			break;
		}

		FStateGraphNodeRef* Node = GetNodeRef(NodeName);
		if (!Node)
		{
			++Removed;
			continue;
		}

		bool bCounted = false;

		switch ((*Node)->Status)
		{
		case FStateGraphNode::EStatus::NotStarted:
		case FStateGraphNode::EStatus::Blocked:
			if (!(*Node)->CheckDependencies())
			{
				if (!StateGraphWeakPtr.IsValid())
				{
					return;
				}

				// Get node again in case CheckDependencies() removed the node.
				Now = FDateTime::UtcNow();
				Node = GetNodeRef(NodeName);
				if (Node)
				{
					(*Node)->SetStatus(FStateGraphNode::EStatus::Blocked);

					if (!StateGraphWeakPtr.IsValid())
					{
						return;
					}

					Now = FDateTime::UtcNow();
					++Blocked;
				}
				else
				{
					++Removed;
				}

				break;
			}

			UE_LOGF(LogStateGraph, Log, "[%ls] Starting node", *(*Node)->LogName);
			(*Node)->SetStatus(FStateGraphNode::EStatus::Started);

			if (!StateGraphWeakPtr.IsValid())
			{
				return;
			}

			Now = FDateTime::UtcNow();
			Node = GetNodeRef(NodeName);
			if (!Node)
			{
				++Removed;
				break;
			}

			(*Node)->StartTime = Now;
			(*Node)->Start();

			if (!StateGraphWeakPtr.IsValid())
			{
				return;
			}

			// Get node again in case Start() removed the node.
			Now = FDateTime::UtcNow();
			Node = GetNodeRef(NodeName);
			if (Node)
			{
				if ((*Node)->Status == FStateGraphNode::EStatus::Completed)
				{
					// Start() called Complete() before returning.
					++Completed;
				}
				else
				{
					++Started;
				}
			}
			else
			{
				++Removed;
			}

			if (!Node || (*Node)->Status != FStateGraphNode::EStatus::Started)
			{
				break;
			}

			bCounted = true;
			// Fall through to check for timeout.

		case FStateGraphNode::EStatus::Started:
			if ((*Node)->Timeout > 0.f)
			{
				const double NodeDuration = (Now - (*Node)->StartTime).GetTotalSeconds();
				const double NodeTimeout = (*Node)->Timeout - NodeDuration;
				if (NodeTimeout <= 0.f)
				{
					++TimedOut;
					UE_LOGF(LogStateGraph, Log, "[%ls] Node timed out (Duration=%.3f Timeout=%.3f)", *(*Node)->LogName, NodeDuration, (*Node)->Timeout);
					(*Node)->SetStatus(FStateGraphNode::EStatus::TimedOut);

					if (!StateGraphWeakPtr.IsValid())
					{
						return;
					}

					Now = FDateTime::UtcNow();
					Node = GetNodeRef(NodeName);
					if (Node)
					{
						(*Node)->TimedOut();
						if (!StateGraphWeakPtr.IsValid())
						{
							return;
						}

						Now = FDateTime::UtcNow();
					}

					break;
				}
				
				if (NextTimeout == 0.f || NodeTimeout < NextTimeout)
				{
					NextTimeout = NodeTimeout;
				}
			}

			if (!bCounted)
			{
				++Running;
			}

			break;

		case FStateGraphNode::EStatus::Completed:
			++Completed;
			break;

		case FStateGraphNode::EStatus::TimedOut:
			++TimedOut;
			break;

		default:
			UE_LOGF(LogStateGraph, Error, "[%ls] Unknown state", *(*Node)->LogName);
			checkNoEntry();
			break;
		}
	}

	bRunning = false;
	UE_LOGF(LogStateGraph, Verbose, "[%ls] Duration=%.3f Timeout=%.3f Blocked=%d Started=%d Running=%d Completed=%d Removed=%d TimedOut=%d",
		*LogName, (Now - StartTime).GetTotalSeconds(), Timeout, Blocked, Started, Running, Completed, Removed, TimedOut);

	if (bRunAgain)
	{
		bRunAgain = false;
		LogDebugInfo(ELogVerbosity::Type::VeryVerbose);
		Run();
		return;
	}

	if (Status != EStatus::Running)
	{
		// State graph was reset or paused during loop, don't change Status.
	}
	else if (Started == 0 && Running == 0)
	{
		if (Blocked == 0 && TimedOut == 0)
		{
			CompletedTime = Now;
			UE_LOGF(LogStateGraph, Log, "[%ls] Completed (Duration=%.3f Timeout=%.3f)", *LogName, GetDuration(), Timeout);
			SetStatus(EStatus::Completed);
		}
		else
		{
			UE_LOGF(LogStateGraph, Warning, "[%ls] Blocked on %d nodes, timed out %d nodes", *LogName, Blocked, TimedOut);
			LogDebugInfo(ELogVerbosity::Type::Warning);
			SetStatus(EStatus::Blocked);
		}
	}
	else
	{
		UE_LOGF(LogStateGraph, VeryVerbose, "[%ls] Waiting on %d nodes", *LogName, Started + Running);
		SetStatus(EStatus::Waiting);
	}

	if (!StateGraphWeakPtr.IsValid())
	{
		return;
	}

	if ((Status == EStatus::Blocked || Status == EStatus::Waiting) && NextTimeout > 0.f)
	{
		UE_LOGF(LogStateGraph, Verbose, "[%ls] Setting timer for %.3f", *LogName, NextTimeout);
		TimeoutTicker = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSPLambda(this, [this](float DeltaTime) {
			Run();
			return false;
		}), NextTimeout);
	}

	LogDebugInfo(ELogVerbosity::Type::VeryVerbose);
}

void FStateGraph::Reset()
{
	// Keep a reference to check if the state graph is destroyed during external functions.
	FStateGraphWeakPtr StateGraphWeakPtr(AsWeak());

	UE_LOGF(LogStateGraph, Verbose, "[%ls] Resetting state graph", *LogName);
	StartTime = 0.f;
	CompletedTime = 0.f;
	bRunAgain = false;
	FTSTicker::RemoveTicker(TimeoutTicker);
	SetStatus(EStatus::NotStarted);

	if (!StateGraphWeakPtr.IsValid())
	{
		return;
	}

	// Copy names since node map may be modified during loop.
	TArray<FName> NodeNames;
	Nodes.GenerateKeyArray(NodeNames);
	for (FName NodeName : NodeNames)
	{
		if (FStateGraphNodeRef* Node = GetNodeRef(NodeName))
		{
			(*Node)->Reset();
			if (!StateGraphWeakPtr.IsValid())
			{
				return;
			}
		}
	}
}

void FStateGraph::Pause()
{
	UE_LOGF(LogStateGraph, Verbose, "[%ls] Pausing state graph", *LogName);
	bRunAgain = false;
	FTSTicker::RemoveTicker(TimeoutTicker);
	SetStatus(EStatus::Paused);
}

void FStateGraph::LogDebugInfo(ELogVerbosity::Type Verbosity)
{
	if (Verbosity > UE_GET_LOG_VERBOSITY(LogStateGraph))
	{
		return;
	}

	FString Message = FString::Printf(TEXT("[%s] Status=%s Nodes=%d Duration=%.3f Timeout=%.3f"), *LogName, GetStatusName(), Nodes.Num(), GetDuration(), Timeout);

	// Dynamic verbosity isn't supported by the UE_LOG macro, so to avoid having to handle every level, we just use Warning and Log.
	// Fatal and Error verbosity levels are not fully supported, as they'll only show up as Warning.
	if (Verbosity <= ELogVerbosity::Type::Warning)
	{
		UE_LOGF(LogStateGraph, Warning, "%ls", *Message);
	}
	else
	{
		UE_LOGF(LogStateGraph, Log, "%ls", *Message);
	}

	for (auto Node : Nodes)
	{
		TMap<FStateGraphNode::EStatus, TArray<FString>> DependenciesByStatus;
		TArray<FString> Missing;

		for (FName Dependency : Node.Value->Dependencies)
		{
			FStateGraphNodeRef* DependencyNode = GetNodeRef(Dependency);
			if (DependencyNode)
			{
				DependenciesByStatus.FindOrAdd((*DependencyNode)->Status).Add(Dependency.ToString());
			}
			else
			{
				Missing.Add(Dependency.ToString());
			}
		}

		TArray<FString> Dependencies;
		if (Missing.Num())
		{
			Dependencies.Add(TEXT("Missing=") + FString::Join(Missing, TEXT(",")));
		}

		for (const TPair<FStateGraphNode::EStatus, TArray<FString>>& Pair : DependenciesByStatus)
		{
			Dependencies.Add(FString::Printf(TEXT("%s=%s"), FStateGraphNode::GetStatusName(Pair.Key), *FString::Join(Pair.Value, TEXT(","))));
		}

		if (Dependencies.Num() == 0)
		{
			Dependencies.Add(TEXT("None"));
		}

		Message = FString::Printf(TEXT("[%s.%s] Status=%s Duration=%.3f Timeout=%.3f Dependencies(%s)"),
			*LogName, *Node.Key.ToString(), Node.Value->GetStatusName(), Node.Value->GetDuration(), Node.Value->Timeout, *FString::Join(Dependencies, TEXT(" ")));

		if (Verbosity <= ELogVerbosity::Type::Warning)
		{
			UE_LOGF(LogStateGraph, Warning, "%ls", *Message);
		}
		else
		{
			UE_LOGF(LogStateGraph, Log, "%ls", *Message);
		}
	}
}

void FStateGraph::SetStatus(EStatus NewStatus)
{
	if (Status != NewStatus)
	{
		const EStatus OldStatus = Status;
		Status = NewStatus;
		OnStatusChanged.Broadcast(*this, OldStatus, NewStatus);
	}
}

void FStateGraph::UpdateConfig()
{
	double RequestedTimeout = 0.;
	GConfig->GetDouble(*ConfigSectionName, TEXT("Timeout"), RequestedTimeout, GEngineIni);
	SetTimeout(RequestedTimeout);
}

void FStateGraph::OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames)
{
	if (IniFilename != GEngineIni)
	{
		return;
	}

	// Keep a reference to check if the state graph is destroyed during external functions.
	FStateGraphWeakPtr StateGraphWeakPtr(AsWeak());

	for (const FString& SectionName : SectionNames)
	{
		// Assume all node section names start with the state graph section name.
		if (!SectionName.StartsWith(ConfigSectionName))
		{
			continue;
		}

		if (SectionName.Len() == ConfigSectionName.Len())
		{
			UpdateConfig();
			continue;
		}

		// Copy names since node map may be modified during loop.
		TArray<FName> NodeNames;
		Nodes.GenerateKeyArray(NodeNames);
		for (FName NodeName : NodeNames)
		{
			FStateGraphNodeRef* Node = GetNodeRef(NodeName);
			if (Node)
			{
				if (SectionName == (*Node)->ConfigSectionName)
				{
					(*Node)->UpdateConfig();
					if (!StateGraphWeakPtr.IsValid())
					{
						return;
					}
				}
			}
		}
	}
}

bool FStateGraph::IsNoTimeouts()
{
	static bool NeverTimeout = FParse::Param(FCommandLine::Get(), TEXT("NoTimeouts"));
	return NeverTimeout;
}

} // UE
