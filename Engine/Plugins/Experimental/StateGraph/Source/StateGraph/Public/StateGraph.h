// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * This module provides a generic state machine framework. For an example, see StateGraphTestsExample.cpp. Features include:
 * - Free-form structure with no required linear or hierarchical structure - just graph nodes with dependencies.
 * - Support for async tasks via a delayed function call to complete nodes.
 * - Modifiable at runtime to add, remove, rename, or reset nodes in the graph.
 * - Able to modify a state graph easily in derived classes or nested objects.
 * - Each node is a full object that can encapsulate all required functions and data.
 * - Wrapper node for delegates to support various function types directly.
 * - Max execution depth of one node at a time.
 * - Timeout support for state graph and individual nodes.
 * - Configurable and hotfixable via ini settings.
 */

#include "Containers/ArrayView.h"
#include "Containers/Ticker.h"
#include "Delegates/Delegate.h"
#include "Logging/LogMacros.h"
#include "Misc/CoreMisc.h"
#include "Misc/DateTime.h"
#include "StateGraphFwd.h"
#include "Templates/IsClass.h"
#include "Templates/IsInvocable.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

#include <type_traits>

#define UE_API STATEGRAPH_API

STATEGRAPH_API DECLARE_LOG_CATEGORY_EXTERN(LogStateGraph, Log, All);

namespace UE {

/**
 * Abstract base class for all nodes in the state graph.
 */
class FStateGraphNode : public TSharedFromThis<FStateGraphNode>, public FNoncopyable
{
public:

	UE_API FStateGraphNode(FName InName);

	UE_API virtual ~FStateGraphNode();

	/** Get the name of the node. */
	FName GetName() const
	{
		return Name;
	}

	const FString& GetLogName() const
	{
		return LogName;
	}

	enum class EStatus : uint8
	{
		/** State graph has not attempted to start this node yet, or the node has been reset. */
		NotStarted,
		/** State graph attempted to start this node, but it is waiting on dependencies or some external blocking condition. */
		Blocked,
		/** State graph has started this node by calling Start(). */
		Started,
		/** Node's Complete() function has been called, allowing dependent nodes to run. */
		Completed,
		/** Node timed out. */
		TimedOut
	};

	/** Get the status of the node. */
	EStatus GetStatus() const
	{
		return Status;
	}

	/** Get the status of the node as a string. */
	const TCHAR* GetStatusName() const
	{
		return GetStatusName(Status);
	}

	static UE_API const TCHAR* GetStatusName(EStatus Status);

	/** Get timeout in seconds. */
	double GetTimeout() const
	{
		return Timeout;
	}

	/** Set timeout in seconds from when Start is run until Complete is run. If this timeout is exceeded, the Status is set to TimedOut. */
	UE_API void SetTimeout(double InTimeout);

	FDateTime GetStartTime() const
	{
		return StartTime;
	}

	FDateTime GetCompletedTime() const
	{
		return CompletedTime;
	}

	/** Get duration of time that the node has taken to run. Returns 0 if not yet started. */
	UE_API double GetDuration() const;

	/** Returns true if all dependencies are met and the node can be started. This base version checks that all nodes in the Dependencies set have been completed, but can be replaced or extended. */
	UE_API virtual bool CheckDependencies() const;

	/** Set Status to Completed and notify the state graph to start other nodes which may have been blocked on this node. */
	UE_API virtual void Complete();

	/** Reset this node to NotStarted so it is run again. Override to reset any state or clean up pending async tasks. */
	UE_API virtual void Reset();

	/** Get the state graph for this node, which may be invalid if this node has not been added or was removed. */
	FStateGraphPtr GetStateGraph() const
	{
		return StateGraphWeakPtr.Pin();
	}

	/**
	 * Helper function to create a new node, add it to this node's StateGraph, and add this node as a dependency.
	 * This allows for chaining nodes together such as:
	 *
	 *	StateGraph->CreateNode("NodeA", this, &SomeMemberFunction)
	 *		->Next("NodeB", &SomeStaticFunction)
	 *		->Next<SomeNodeClass>()
	 *		->Next("NodeC", [](FStateGraph& StateGraph, FStateGraphNodeFunctionComplete Complete) { Complete(); });
	 */
	template<typename NodeType, typename... ArgsTypes>
	TSharedPtr<NodeType, ESPMode::ThreadSafe> Next(ArgsTypes&&... Args) const;

	template<typename... ArgsTypes>
	FStateGraphNodeFunctionPtr Next(ArgsTypes&&... Args) const;

	TSet<FName> Dependencies;

protected:

	/** Start this node. */
	virtual void Start() = 0;

	/** Node timed out. Override to cleanup up any state or pending async tasks. This will not get called if OnNodeStatusChanged removes the node after updating the status to TimedOut. */
	virtual void TimedOut() {}

	/** Node was removed from the state graph. Override to cleanup up any state or pending async tasks. */
	virtual void Removed() {}

	/** Update config from INI files. Called when node is added to a state graph or during hotfixes. Override this to read any custom config the for the node. */
	UE_API virtual void UpdateConfig();

	/** Get the name of the node. */
	const FString& GetConfigSectionName() const
	{
		return ConfigSectionName;
	}

private:

	/** Set status and fire delegate if needed. */
	UE_API void SetStatus(EStatus NewStatus);

	FName Name;
	FString LogName;
	FString ConfigSectionName;
	EStatus Status = EStatus::NotStarted;
	FStateGraphWeakPtr StateGraphWeakPtr;
	FDateTime StartTime = 0.f;
	FDateTime CompletedTime = 0.f;
	double Timeout = 0.f;

	friend FStateGraph;
};

/** Start function delegate type for FStateGraphNodeFunction nodes. */
DECLARE_DELEGATE_TwoParams(FStateGraphNodeFunctionStart, FStateGraph& /*StateGraph*/, FStateGraphNodeFunctionComplete /*Complete*/);

/**
 * Node for wrapping functions as delegates.
 */
class FStateGraphNodeFunction : public FStateGraphNode
{
public:

	UE_API FStateGraphNodeFunction(FName InName, const FStateGraphNodeFunctionStart& InStartFunction);
	UE_API virtual bool CheckDependencies() const override;

protected:

	UE_API virtual void Start() override;
	FStateGraphNodeFunctionStart StartFunction;

	friend FStateGraph;
};

/**
 * State graph class to run and manage nodes with.
 */
class FStateGraph : public TSharedFromThis<FStateGraph>, public FNoncopyable
{
public:

	UE_API FStateGraph(FName InName, const FString& InContextName = FString());

	UE_API virtual ~FStateGraph();

	/** Perform state graph initialization that could not run in constructor due to shared pointers not being ready, such as registering for hotfixes. */
	UE_API void Initialize();

	/** Get the name of the state graph. */
	FName GetName() const
	{
		return Name;
	}

	const FString& GetContextName() const
	{
		return ContextName;
	}

	const FString& GetLogName() const
	{
		return LogName;
	}

	enum class EStatus : uint8
	{
		/** State graph has not started yet. */
		NotStarted,
		/** State graph is running. */
		Running,
		/** State graph is waiting for one or more nodes to complete. */
		Waiting,
		/** State graph is blocked due to missing dependencies. */
		Blocked,
		/** State graph has completed. */
		Completed,
		/** State graph has been paused. */
		Paused,
		/** State graph timed out and stopped running. */
		TimedOut
	};

	/** Get the status of the state graph. */
	EStatus GetStatus() const
	{
		return Status;
	}

	/** Get the status of the state graph as a string. */
	const TCHAR* GetStatusName() const
	{
		return GetStatusName(Status);
	}

	static UE_API const TCHAR* GetStatusName(EStatus Status);

	/** Delegate fired when the state graph status changes. Useful to know when it becomes blocked or times out. */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FStatusChanged, FStateGraph& /*StateGraph*/, EStatus /*OldStatus*/, EStatus /*NewStatus*/ );
	FStatusChanged OnStatusChanged;

	/** Delegate fired when a node status changes. */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FNodeStatusChanged, FStateGraphNode& /*Node*/, FStateGraphNode::EStatus /*OldStatus*/, FStateGraphNode::EStatus /*NewStatus*/ );
	FNodeStatusChanged OnNodeStatusChanged;

	/** Get timeout in seconds. */
	double GetTimeout() const
	{
		return Timeout;
	}

	/** Set timeout in seconds from the initial run call (when Status changes from NotStarted to Running) to when Status changes to Completed. */
	UE_API void SetTimeout(double InTimeout);

	FDateTime GetStartTime() const
	{
		return StartTime;
	}

	FDateTime GetCompletedTime() const
	{
		return CompletedTime;
	}

	/** Get duration of time that the state graph has taken to run. Returns 0 if not yet started. */
	UE_API double GetDuration() const;

	/** Add a node to the state graph. Returns true if successful, false if the node is either already added to a state graph or if an existing node with the same name exists. */
	UE_API bool AddNode(const FStateGraphNodeRef& Node);

	/** Remove a node from the state graph. Returns true if successful, false if no node was found for the name. */
	UE_API bool RemoveNode(FName NodeName);

	/** Remove all nodes from the state graph that exist when this function is started. It's possible for nodes to be added back before this function returns if Node->Removed() adds nodes. */
	UE_API void RemoveAllNodes();

	/** Get map of all nodes in the state graph. References should not be cached in case the node is destroyed. */
	TMap<FName, FStateGraphNodeRef>& GetNodes()
	{
		return Nodes;
	}

	/** Get a node from the state graph. References should not be cached in case the node is destroyed. */
	FStateGraphNodeRef* GetNodeRef(FName NodeName)
	{
		return Nodes.Find(NodeName);
	}

	/** Get a node from the state graph, casting to a derived type if needed. */
	template<typename NodeType = FStateGraphNode>
	TSharedPtr<NodeType, ESPMode::ThreadSafe> GetNode(FName NodeName)
	{
		TSharedPtr<NodeType, ESPMode::ThreadSafe> NodePtr;
		if (FStateGraphNodeRef* NodeRef = GetNodeRef(NodeName))
		{
			NodePtr = StaticCastSharedRef<NodeType>(*NodeRef);
		}

		return NodePtr;
	}

	/** Create and add a new node for a given node class type. */
	template<typename NodeType, typename... ArgsTypes>
	TSharedRef<NodeType, ESPMode::ThreadSafe> CreateNode(ArgsTypes&&... Args)
	{
		TSharedRef<NodeType, ESPMode::ThreadSafe> Node(MakeShared<NodeType>(Forward<ArgsTypes>(Args)...));
		ensureAlways(AddNode(Node));
		return Node;
	}

	/** Create a new function node for member functions. */
	template<typename ObjectType, typename FunctionType, typename... ArgsTypes>
	typename TEnableIf<TIsClass<ObjectType>::Value && TIsInvocable<FunctionType, ObjectType, FStateGraph&, FStateGraphNodeFunctionComplete, ArgsTypes...>::Value, FStateGraphNodeFunctionRef>::Type
		CreateNode(FName NodeName, ObjectType* Object, FunctionType Function, ArgsTypes&&... Args)
	{
		if constexpr (IsDerivedFromSharedFromThis<ObjectType>())
		{
			return CreateNode<FStateGraphNodeFunction>(NodeName, FStateGraphNodeFunctionStart::CreateSP(Object, Function, Forward<ArgsTypes>(Args)...));
		}
		else if constexpr (TIsDerivedFrom<ObjectType, UObject>::IsDerived)
		{
			return CreateNode<FStateGraphNodeFunction>(NodeName, FStateGraphNodeFunctionStart::CreateUObject(Object, Function, Forward<ArgsTypes>(Args)...));
		}
		else if constexpr (std::is_member_function_pointer_v<FunctionType>)
		{
			return CreateNode<FStateGraphNodeFunction>(NodeName, FStateGraphNodeFunctionStart::CreateRaw(Object, Function, Forward<ArgsTypes>(Args)...));
		}
	}

	/** Create a new function node for lambdas tied to objects. */
	template<typename ObjectType, typename FunctionType, typename... ArgsTypes>
	typename TEnableIf<TIsClass<ObjectType>::Value && TIsInvocable<FunctionType, FStateGraph&, FStateGraphNodeFunctionComplete, ArgsTypes...>::Value, FStateGraphNodeFunctionRef>::Type
		CreateNode(FName NodeName, ObjectType* Object, FunctionType Function, ArgsTypes&&... Args)
	{
		if constexpr (IsDerivedFromSharedFromThis<ObjectType>())
		{
			return CreateNode<FStateGraphNodeFunction>(NodeName, FStateGraphNodeFunctionStart::CreateSPLambda(Object, Function, Forward<ArgsTypes>(Args)...));
		}
		else if constexpr (TIsDerivedFrom<ObjectType, UObject>::IsDerived)
		{
			return CreateNode<FStateGraphNodeFunction>(NodeName, FStateGraphNodeFunctionStart::CreateWeakLambda(Object, Function, Forward<ArgsTypes>(Args)...));
		}
	}

	/** Create a new function node for lambas and static methods. */
	template<typename FunctionType, typename... ArgsTypes>
	typename TEnableIf<TIsInvocable<FunctionType, FStateGraph&, FStateGraphNodeFunctionComplete, ArgsTypes...>::Value, FStateGraphNodeFunctionRef>::Type
		CreateNode(FName NodeName, FunctionType Function, ArgsTypes&&... Args)
	{
		return CreateNode<FStateGraphNodeFunction>(NodeName, FStateGraphNodeFunctionStart::CreateLambda(Function, Forward<ArgsTypes>(Args)...));
	}

	/** Helper function for adding dependencies to a node. */
	UE_API bool AddDependencies(FName NodeName, const TArrayView<const FName> Dependencies);

	/**
	 * Run the state graph. This is run automatically when nodes complete to try starting any blocked nodes, so this only needs to be run manually when:
	 * - The state graph has not been run yet (Status = NotStarted).
	 * - The state graph has become blocked, and the blocking condition has been resolved (Status = Blocked).
	 * - A new node has been added to the state graph or dependencies for nodes have changed that may unblock existing nodes.
	 * - The state graph was reset or paused and should be started again.
	 */
	UE_API void Run();

	/**
	 * Stop and reset the state graph and all nodes. The state graph can be started again with Run().
	 */
	UE_API void Reset();

	/**
	 * Pause the state graph and prevent additional nodes from being started. Other actions can still happen such as nodes completing nodes, adding nodes, resetting, etc.
	 */
	UE_API void Pause();

	/** Log debug info for the state graph and each node. */
	UE_API void LogDebugInfo(ELogVerbosity::Type Verbosity);

protected:

	static bool IsNoTimeouts();
	friend class FStateGraphNode;

	/** Set status and fire delegate if needed. */
	UE_API void SetStatus(EStatus NewStatus);

	/** Update config from INI files. Called from Initialize and during hotfixes. */
	UE_API void UpdateConfig();

	/** Callback to update state graph and node configuration during hotfixes. */
	UE_API void OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames);

	FName Name;
	FString ContextName;
	FString LogName;
	FString ConfigSectionName;
	EStatus Status = EStatus::NotStarted;
	FDateTime StartTime = 0.f;
	FDateTime CompletedTime = 0.f;
	double Timeout = 0.f;
	TMap<FName, FStateGraphNodeRef> Nodes;
	bool bRunning = false;
	bool bRunAgain = false;
	FTSTicker::FDelegateHandle TimeoutTicker;
	FDelegateHandle ConfigSectionsChangedDelegate;
};

// FStateGraphNode templates that need FStateGraph defined.

template<typename NodeType, typename... ArgsTypes>
TSharedPtr<NodeType, ESPMode::ThreadSafe> FStateGraphNode::Next(ArgsTypes&&... Args) const
{
	FStateGraphPtr StateGraphPtr = GetStateGraph();
	if (!StateGraphPtr)
	{
		UE_LOGF(LogStateGraph, Warning, "[%ls] Node attempted to be created with invalid state graph", *LogName);
		return TSharedPtr<NodeType, ESPMode::ThreadSafe>();
	}

	TSharedPtr<NodeType, ESPMode::ThreadSafe> Node(StateGraphPtr->CreateNode<NodeType>(Forward<ArgsTypes>(Args)...));
	Node->Dependencies.Add(Name);
	return Node;
}

template<typename... ArgsTypes>
FStateGraphNodeFunctionPtr FStateGraphNode::Next(ArgsTypes&&... Args) const
{
	FStateGraphPtr StateGraphPtr = GetStateGraph();
	if (!StateGraphPtr)
	{
		UE_LOGF(LogStateGraph, Warning, "[%ls] Node attempted to be created with invalid state graph", *LogName);
		return FStateGraphNodeFunctionPtr();
	}

	FStateGraphNodeFunctionPtr Node(StateGraphPtr->CreateNode(Forward<ArgsTypes>(Args)...));
	Node->Dependencies.Add(Name);
	return Node;
}

} // UE

#undef UE_API

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Templates/IsMemberPointer.h"
#endif
