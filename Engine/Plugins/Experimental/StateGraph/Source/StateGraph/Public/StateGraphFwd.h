// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

namespace UE {

class FStateGraph;
using FStateGraphRef = TSharedRef<FStateGraph, ESPMode::ThreadSafe>;
using FStateGraphPtr = TSharedPtr<FStateGraph, ESPMode::ThreadSafe>;
using FStateGraphWeakPtr = TWeakPtr<FStateGraph, ESPMode::ThreadSafe>;

class FStateGraphNode;
using FStateGraphNodeRef = TSharedRef<FStateGraphNode, ESPMode::ThreadSafe>;
using FStateGraphNodePtr = TSharedPtr<FStateGraphNode, ESPMode::ThreadSafe>;
using FStateGraphNodeWeakPtr = TWeakPtr<FStateGraphNode, ESPMode::ThreadSafe>;

class FStateGraphNodeFunction;
using FStateGraphNodeFunctionRef = TSharedRef<FStateGraphNodeFunction, ESPMode::ThreadSafe>;
using FStateGraphNodeFunctionPtr = TSharedPtr<FStateGraphNodeFunction, ESPMode::ThreadSafe>;
using FStateGraphNodeFunctionWeakPtr = TWeakPtr<FStateGraphNodeFunction, ESPMode::ThreadSafe>;

/** Complete function type passed into FStateGraphNodeFunction node start functions. */
using FStateGraphNodeFunctionComplete = TFunction<void()>;

} // UE
