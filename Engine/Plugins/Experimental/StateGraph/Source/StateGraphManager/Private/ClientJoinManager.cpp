// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientJoinManager.h"
#include "StateGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClientJoinManager)

namespace UE::ClientJoin::Name
{
const FName StateGraph("ClientJoin");
} // UE::ClientJoin::Name
