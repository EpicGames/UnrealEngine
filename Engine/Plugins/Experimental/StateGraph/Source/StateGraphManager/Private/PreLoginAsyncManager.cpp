// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreLoginAsyncManager.h"

#include "GameDelegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PreLoginAsyncManager)

namespace UE::PreLoginAsync::Name
{
	const FName StateGraph("PreLoginAsync");
	const FName Options("Options");
} // UE::PreLoginAsync::Name

#if WITH_SERVER_CODE

static TAutoConsoleVariable<bool> CVarFailPreloginOnPlayerConnectionLost(
	TEXT("PreloginAsync.FailPreloginOnPlayerConnectionLost"),
	true,
	TEXT("Prelogin stategraph will complete as failed if player's connection is lost before it completes"),
	ECVF_Default);

void UPreLoginAsyncManager::InitializeStateGraph(UE::FStateGraph& StateGraph, const FString& Options, const FString& Address, const FUniqueNetIdRepl& UniqueId, const AGameModeBase::FOnPreLoginCompleteDelegate& OnComplete)
{
	UE::FStateGraphRef* OldStateGraph = RunningStateGraphs.Find(UniqueId);
	if (OldStateGraph)
	{
		UE_LOGF(LogStateGraph, Warning, "[%ls] Duplicate login detected", *StateGraph.GetContextName());
		CompleteLogin((*OldStateGraph).Get(), TEXT("Duplicate login detected"));
	}

	RunningStateGraphs.Emplace(UniqueId, StateGraph.AsShared());

	StateGraph.OnStatusChanged.AddWeakLambda(this,
		[this](UE::FStateGraph& StateGraph, UE::FStateGraph::EStatus OldStatus, UE::FStateGraph::EStatus NewStatus)
		{
			if (NewStatus == UE::FStateGraph::EStatus::Completed)
			{
				CompleteLogin(StateGraph, FString());
			}
			else if (NewStatus == UE::FStateGraph::EStatus::Blocked)
			{
				CompleteLogin(StateGraph, TEXT("PreLoginAsync blocked"));
			}
			else if (NewStatus == UE::FStateGraph::EStatus::TimedOut)
			{
				CompleteLogin(StateGraph, TEXT("PreLoginAsync timed out"));
			}
		});

	StateGraph.CreateNode<UE::PreLoginAsync::FOptions>(this, Options, Address, UniqueId, OnComplete);
	if (CVarFailPreloginOnPlayerConnectionLost.GetValueOnAnyThread())
	{
		FGameDelegates::Get().GetPendingConnectionLostDelegate().AddSPLambda(&StateGraph, [UniqueId, WeakStateGraph = StateGraph.AsWeak()](const FUniqueNetIdRepl& ConnectionUniqueId)
			{
				if (UniqueId == ConnectionUniqueId)
				{
					UE::FStateGraphPtr StateGraph = WeakStateGraph.Pin();
					FGameDelegates::Get().GetPendingConnectionLostDelegate().RemoveAll(StateGraph.Get());
					CompleteLogin(*StateGraph, TEXT("User disconnected"));
				}
			});
	}
}

void UPreLoginAsyncManager::CompleteLogin(UE::FStateGraph& StateGraph, const FString& Error)
{
	UE_LOGF(LogStateGraph, Log, "[%ls] CompleteLogin: %ls", *StateGraph.GetContextName(), Error.IsEmpty() ? TEXT("Success") : *Error);

	UE::PreLoginAsync::FOptionsPtr Options = UE::PreLoginAsync::FOptions::Get(StateGraph);
	if (Options)
	{
		Options->OnComplete.ExecuteIfBound(Error);
		TObjectPtr<UPreLoginAsyncManager> Manager = Options->WeakManager.Get();
		if (Manager)
		{
			Manager->RunningStateGraphs.Remove(Options->UniqueId);
		}
	}
}

#endif // WITH_SERVER_CODE
