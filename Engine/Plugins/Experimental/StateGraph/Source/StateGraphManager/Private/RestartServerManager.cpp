// Copyright Epic Games, Inc. All Rights Reserved.

#include "RestartServerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RestartServerManager)

namespace UE::RestartServer::Name
{
	const FName StateGraph("RestartServer");
} // UE::RestartServer::Name
