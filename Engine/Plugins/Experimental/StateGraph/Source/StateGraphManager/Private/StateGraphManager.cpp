// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateGraphManager.h"

#include "Modules/ModuleManager.h"
#include "StateGraph.h"

class FStateGraphManagerModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FStateGraphManagerModule, StateGraphManager);

namespace UE
{

FStateGraphManager::~FStateGraphManager()
{
}

void FStateGraphManager::AddCreateDelegate(const FStateGraphManagerCreateDelegate& Delegate)
{
	CreateDelegates.Add(Delegate);
}

UE::FStateGraphPtr FStateGraphManager::Create(const FString& ContextName)
{
	UE::FStateGraphPtr StateGraph = MakeShared<UE::FStateGraph>(GetStateGraphName(), ContextName);
	StateGraph->Initialize();

	for (int32 Index = 0; Index < CreateDelegates.Num(); Index++)
	{
		FStateGraphManagerCreateDelegate& Delegate = CreateDelegates[Index];
		if (!Delegate.IsBound())
		{
			// Cleanup stale delegates.
			CreateDelegates.RemoveAt(Index--);
		}
		else if (!Delegate.Execute(*StateGraph.Get()))
		{
			StateGraph.Reset();
			break;
		}
	}

	return StateGraph;
}

FStateGraphManagerTracked::~FStateGraphManagerTracked()
{
}

UE::FStateGraphPtr FStateGraphManagerTracked::Create(const FString& ContextName)
{
	if (StateGraphs.Find(ContextName))
	{
		UE_LOGF(LogStateGraph, Warning, "Failed to add duplicate state graph for context: %ls", *ContextName);
		return nullptr;
	}

	UE::FStateGraphPtr Stategraph = UE::FStateGraphManager::Create(ContextName);
	if (!Stategraph.IsValid())
	{
		UE_LOGF(LogStateGraph, Warning, "Failed to create state graph for context: %ls", *ContextName);
		return nullptr;
	}

	return StateGraphs.Emplace(ContextName, Stategraph.ToSharedRef());
}

UE::FStateGraphPtr FStateGraphManagerTracked::Find(const FString& ContextName) const
{
	const UE::FStateGraphRef* StateGraph = StateGraphs.Find(ContextName);
	if (StateGraph)
	{
		return *StateGraph;
	}

	return nullptr;
}

void FStateGraphManagerTracked::Remove(const FString& ContextName)
{
	if (StateGraphs.Remove(ContextName) == 0)
	{
		UE_LOGF(LogStateGraph, Warning, "Failed to remove state graph for context: %ls", *ContextName);
	}
}

} // UE
