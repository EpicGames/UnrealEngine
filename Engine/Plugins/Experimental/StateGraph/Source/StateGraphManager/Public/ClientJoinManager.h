// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * State graph manager for a client when joining a server.
 */

#include "StateGraphManager.h"
#include "Subsystems/GameInstanceSubsystem.h"

#include "ClientJoinManager.generated.h"

namespace UE::ClientJoin::Name
{
STATEGRAPHMANAGER_API extern const FName StateGraph;
} // UE::ClientJoin::Name

/** Subsystem manager that other modules and subsystems can depend on to add ClientJoin state graph delegates with. */
UCLASS(MinimalAPI)
class UClientJoinManager : public UGameInstanceSubsystem, public UE::FStateGraphManagerTracked
{
	GENERATED_BODY()

public:
	virtual FName GetStateGraphName() const override
	{
		return UE::ClientJoin::Name::StateGraph;
	}
};
