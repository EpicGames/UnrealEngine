// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * State graph manager for AGameModeBase::PreLoginAsync.
 */

#include "Containers/Map.h"
#include "GameFramework/GameModeBase.h"
#include "GameFramework/OnlineReplStructs.h"
#include "StateGraph.h"
#include "StateGraphManager.h"
#include "Subsystems/WorldSubsystem.h"

#include "PreLoginAsyncManager.generated.h"

#define UE_API STATEGRAPHMANAGER_API

class UPreLoginAsyncManager;

namespace UE::PreLoginAsync
{

namespace Name
{
STATEGRAPHMANAGER_API extern const FName StateGraph;
STATEGRAPHMANAGER_API extern const FName Options;
} // Name

#if WITH_SERVER_CODE

class FOptions;
using FOptionsPtr = TSharedPtr<FOptions, ESPMode::ThreadSafe>;

/** State graph node that is used to store options that PreLoginAsync was called with. */
class FOptions : public UE::FStateGraphNode
{
public:
	static FOptionsPtr Get(UE::FStateGraph& StateGraph)
	{
		return StateGraph.GetNode<FOptions>(Name::Options);
	}

	FOptions(TObjectPtr<UPreLoginAsyncManager> InManager, const FString& InOptions, const FString& InAddress, const FUniqueNetIdRepl& InUniqueId, const AGameModeBase::FOnPreLoginCompleteDelegate& InOnComplete) :
		UE::FStateGraphNode(Name::Options),
		WeakManager(InManager),
		Options(InOptions),
		Address(InAddress),
		UniqueId(InUniqueId),
		OnComplete(InOnComplete)
	{
	}

	virtual void Start() override
	{
		// Nothing to do, this node just holds options.
		Complete();
	}

	TWeakObjectPtr<UPreLoginAsyncManager> WeakManager;
	FString Options;
	FString Address;
	FUniqueNetIdRepl UniqueId;
	AGameModeBase::FOnPreLoginCompleteDelegate OnComplete;
};

#endif //WITH_SERVER_CODE

} // UE::PreLoginAsync

/** Subsystem manager that other modules and subsystems can depend on to register PreLoginAsync state graph delegates with. */
UCLASS(MinimalAPI)
class UPreLoginAsyncManager : public UWorldSubsystem, public UE::FStateGraphManager
{
	GENERATED_BODY()

public:
	virtual FName GetStateGraphName() const override
	{
		return UE::PreLoginAsync::Name::StateGraph;
	}

#if WITH_SERVER_CODE
	/**
	 * Add the state graph to the map of running requests, detecting and removing conflicts if needed. This also creates the options node from the original
	 * PreLoginAsync options. This isn't part of a Create() override so the OnComplete delegate can use the created state graph if needed.
	 */
	UE_API void InitializeStateGraph(UE::FStateGraph& StateGraph, const FString& Options, const FString& Address, const FUniqueNetIdRepl& UniqueId, const AGameModeBase::FOnPreLoginCompleteDelegate& OnComplete);

	/**
	 * Call the OnComplete delegate to finish the PreLoginAsync call and remove the state graph from the map of running requests. As long as no references
	 * are kept to the shared pointer, this will free the state graph and all nodes. The state graph should no longer be used after this is called.
	 */
	static UE_API void CompleteLogin(UE::FStateGraph& StateGraph, const FString& Error);

private:
	/** Map of pending PreLoginAsync requests currently running. Only one per NetId is allowed. */
	TMap<FUniqueNetIdRepl, UE::FStateGraphRef> RunningStateGraphs;

#endif //WITH_SERVER_CODE
};

#undef UE_API
