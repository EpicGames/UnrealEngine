// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * State graph manager for AGameSession::RegisterServer.
 */

#include "StateGraphManager.h"
#include "Subsystems/WorldSubsystem.h"

#include "RegisterServerManager.generated.h"

namespace UE::RegisterServer::Name
{
STATEGRAPHMANAGER_API extern const FName StateGraph;
} // UE::RegisterServer::Name

/** Subsystem manager that other modules and subsystems can depend on to add RegisterServer state graph delegates with. */
UCLASS(MinimalAPI)
class URegisterServerManager : public UWorldSubsystem, public UE::FStateGraphManagerTracked
{
	GENERATED_BODY()

public:
	virtual FName GetStateGraphName() const override
	{
		return UE::RegisterServer::Name::StateGraph;
	}
};
