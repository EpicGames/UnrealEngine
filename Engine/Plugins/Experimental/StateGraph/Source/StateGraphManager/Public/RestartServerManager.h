// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * State graph manager for AGameSession::Restart.
 */

#include "StateGraphManager.h"
#include "Subsystems/WorldSubsystem.h"

#include "RestartServerManager.generated.h"

namespace UE::RestartServer::Name
{
STATEGRAPHMANAGER_API extern const FName StateGraph;
} // UE::RestartServer::Name

/** Subsystem manager that other modules and subsystems can depend on to add RestartServer state graph delegates with. */
UCLASS(MinimalAPI)
class URestartServerManager : public UWorldSubsystem, public UE::FStateGraphManagerTracked
{
	GENERATED_BODY()

public:
	virtual FName GetStateGraphName() const override
	{
		return UE::RestartServer::Name::StateGraph;
	}
};
