// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * This module provides managers to dynamically configure and create state graphs for various use cases.
 * This includes a set of common managers for engine-level classes, but plugins and game code can define
 * their own by creating subsystems that inherit from FStateGraphManager below.
 */

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "StateGraphFwd.h"
#include "UObject/NameTypes.h"

#define UE_API STATEGRAPHMANAGER_API

namespace UE
{

DECLARE_DELEGATE_RetVal_OneParam(bool, FStateGraphManagerCreateDelegate, UE::FStateGraph& /*StateGraph*/);

class FStateGraphManager
{
public:
	UE_API virtual ~FStateGraphManager();
	virtual FName GetStateGraphName() const = 0;

	/** Add a delegate to be called when creating a new state graph instance. These are called in the order they were added. */
	UE_API virtual void AddCreateDelegate(const FStateGraphManagerCreateDelegate& Delegate);

	/** Create an untracked state graph instance. */
	UE_API virtual UE::FStateGraphPtr Create(const FString& ContextName = FString());

protected:
	/** Array of registered delegates that are called in order when creating a new state graph instance. */
	TArray<FStateGraphManagerCreateDelegate> CreateDelegates;
};

class FStateGraphManagerTracked : public FStateGraphManager
{
public:
	UE_API virtual ~FStateGraphManagerTracked();

	/** Create a tracked state graph instance by ContextName. Only one instance per context is allowed, so the context should be a unique ID representing that instance. */
	UE_API virtual UE::FStateGraphPtr Create(const FString& ContextName = FString()) override;

	/** Find a tracked state graph instance by ContextName. */
	UE_API virtual UE::FStateGraphPtr Find(const FString& ContextName = FString()) const;

	/** Remove a tracked state graph instance by ContextName, if one exists. */
	UE_API virtual void Remove(const FString& ContextName = FString());

protected:
	/** Map of state graphs currently tracked by context name. */
	TMap<FString, UE::FStateGraphRef> StateGraphs;
};

} // UE

#undef UE_API
