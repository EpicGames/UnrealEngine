// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StateGraphManager : ModuleRules
{
	public StateGraphManager(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[] {
			"Core",
			"Engine",
			"StateGraph"
		});

		PrivateDependencyModuleNames.AddRange(new string[] {
			"CoreUObject"
		});
	}
}
