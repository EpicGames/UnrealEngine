{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "State Graph",
	"Description": "Generic state machine management class.",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "StateGraph",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "StateGraphManager",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	]
}
